// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CEF3Utils : ModuleRules
{
	public CEF3Utils(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64
		||  Target.Platform == UnrealTargetPlatform.Mac
		||  Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"CEF3"
				);
		}
	}
}
