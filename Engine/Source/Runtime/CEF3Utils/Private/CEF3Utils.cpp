// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF3Utils.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Misc/OutputDeviceFile.h"
#include "CEF3UtilsLog.h"

#if WITH_CEF3
#  include "include/cef_version.h"
#  define CEF_VERSION_DIR TEXT("/") TEXT(CEF_VERSION)
#  if CEF3_USE_EXPERIMENTAL_VERSION
#    define CEF_VERSION_SUFFIX TEXT("")
#  else
#    define CEF_VERSION_SUFFIX TEXT("+v2")
#  endif
#  if PLATFORM_MAC
#    include "include/wrapper/cef_library_loader.h"
#    define CEF3_BIN_DIR TEXT("Binaries/ThirdParty/CEF3/Mac")
#    define CEF3_FRAMEWORK_DIR CEF3_BIN_DIR CEF_VERSION_DIR TEXT("/Chromium Embedded Framework.framework")
#    define CEF3_FRAMEWORK_EXE CEF3_FRAMEWORK_DIR TEXT("/Chromium Embedded Framework")

#    define CEF3_BUNDLE_DIR TEXT("../Frameworks/Chromium Embedded Framework.framework")
#    define CEF3_BUNDLE_EXE CEF3_BUNDLE_DIR TEXT("/Chromium Embedded Framework")
#  endif // PLATFORM_MAC
#endif // WITH_CEF3

DEFINE_LOG_CATEGORY(LogCEF3Utils);

IMPLEMENT_MODULE(FDefaultModuleImpl, CEF3Utils);

#if WITH_CEF3
namespace CEF3Utils
{
#if PLATFORM_WINDOWS
	void* CEF3DLLHandle = nullptr;
	void* ElfHandle = nullptr;
	void* D3DHandle = nullptr;
	void* GLESHandle = nullptr;
	void* EGLHandle = nullptr;
	FString DllPath;
#elif PLATFORM_MAC
	// Dynamically load the CEF framework library.
	CefScopedLibraryLoader *CEFLibraryLoader = nullptr;
	FString FrameworkPath;
#endif

	void* LoadDllCEF(const FString& Path)
	{
		if (Path.IsEmpty())
		{
			return nullptr;
		}
		void* Handle = FPlatformProcess::GetDllHandle(*Path);
		if (!Handle)
		{
			int32 ErrorNum = FPlatformMisc::GetLastError();
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, ErrorNum);
			UE_LOGF(LogCEF3Utils, Error, "Failed to get CEF3 DLL handle for %ls: %ls (%d)", *Path, ErrorMsg, ErrorNum);
		}
		return Handle;
	}

	bool LoadCEF3Modules(bool bIsMainApp)
	{
#if PLATFORM_WINDOWS
	#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		DllPath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Binaries/ThirdParty/CEF3/WinArm64"), CEF_VERSION_DIR);
	#else
		DllPath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Binaries/ThirdParty/CEF3/Win64"), CEF_VERSION_DIR CEF_VERSION_SUFFIX);
	#endif

		FPlatformProcess::PushDllDirectory(*DllPath);
		CEF3DLLHandle = LoadDllCEF(FPaths::Combine(*DllPath, TEXT("libcef.dll")));
		if (CEF3DLLHandle)
		{
			ElfHandle = LoadDllCEF(FPaths::Combine(*DllPath, TEXT("chrome_elf.dll")));
			D3DHandle = LoadDllCEF(FPaths::Combine(*DllPath, TEXT("d3dcompiler_47.dll")));
			GLESHandle = LoadDllCEF(FPaths::Combine(*DllPath, TEXT("libGLESv2.dll")));
			EGLHandle = LoadDllCEF(FPaths::Combine(*DllPath, TEXT("libEGL.dll")));
		}
		FPlatformProcess::PopDllDirectory(*DllPath);
		DllPath = FPaths::ConvertRelativePathToFull(DllPath);
		return CEF3DLLHandle != nullptr;
#elif PLATFORM_MAC
		// Dynamically load the CEF framework library.
		CEFLibraryLoader = new CefScopedLibraryLoader();
		
		// look for proper framework bundle, and failing that, fall back to old location
		FrameworkPath = FPaths::Combine(FPaths::GetPath(FPlatformProcess::ExecutablePath()), CEF3_BUNDLE_EXE);
		if (!FPaths::FileExists(FrameworkPath))
		{
			FrameworkPath = (FPaths::Combine(*FPaths::EngineDir(), CEF3_FRAMEWORK_EXE));
		}
		FrameworkPath = FPaths::ConvertRelativePathToFull(FrameworkPath);

		bool bLoaderInitialized = false;
		if (bIsMainApp)
		{
			// first look in standard Frameworks dir, then loom in old UE path
			bLoaderInitialized = CEFLibraryLoader->LoadInMain(TCHAR_TO_ANSI(*FrameworkPath));
			if (!bLoaderInitialized)
			{
				UE_LOGF(LogCEF3Utils, Error, "Chromium loader initialization failed");
			}
		}
		else
		{
			bLoaderInitialized = CEFLibraryLoader->LoadInHelper(TCHAR_TO_ANSI(*FrameworkPath));
			if (!bLoaderInitialized)
			{
				UE_LOGF(LogCEF3Utils, Error, "Chromium helper loader initialization failed");
			}
		}
		return bLoaderInitialized;
#elif PLATFORM_LINUX
		return true;
#else
		// unsupported platform for libcef
		return false;
#endif
	}

	void UnloadCEF3Modules()
	{
#if PLATFORM_WINDOWS
		FPlatformProcess::FreeDllHandle(CEF3DLLHandle);
		CEF3DLLHandle = nullptr;
		FPlatformProcess::FreeDllHandle(ElfHandle);
		ElfHandle = nullptr;
		FPlatformProcess::FreeDllHandle(D3DHandle);
		D3DHandle = nullptr;
		FPlatformProcess::FreeDllHandle(GLESHandle);
		GLESHandle = nullptr;
		FPlatformProcess::FreeDllHandle(EGLHandle);
		EGLHandle = nullptr;
#elif PLATFORM_MAC
		delete CEFLibraryLoader;
		CEFLibraryLoader = nullptr;
#endif
	}

#if PLATFORM_WINDOWS
	CEF3UTILS_API void* GetCEF3ModuleHandle()
	{
		return CEF3DLLHandle;
	}
#endif

	FString GetCEF3ModulePath()
	{
#if PLATFORM_WINDOWS
		return DllPath;
#elif PLATFORM_MAC
		return FrameworkPath;
#elif PLATFORM_LINUX
		return FString();
#else
#endif
	}

	void BackupCEF3Logfile(const FString& LogFilePath)
	{
		const FString Cef3LogFile = FPaths::Combine(*LogFilePath,TEXT("cef3.log"));
		IFileManager& FileManager = IFileManager::Get();
		if (FileManager.FileSize(*Cef3LogFile) > 0) // file exists and is not empty
		{
			FString Name, Extension;
			FString(Cef3LogFile).Split(TEXT("."), &Name, &Extension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			FDateTime OriginalTime = FileManager.GetTimeStamp(*Cef3LogFile);
			FString BackupFilename = FString::Printf(TEXT("%s%s%s.%s"), *Name, BACKUP_LOG_FILENAME_POSTFIX, *OriginalTime.ToString(), *Extension);
			// do not retry resulting in an error if log still in use
			if (!FileManager.Move(*BackupFilename, *Cef3LogFile, false, false, false, true))
			{
				UE_LOGF(LogCEF3Utils, Warning, "Failed to backup cef3.log");
			}
		}
	}
};
#endif //WITH_CEF3
