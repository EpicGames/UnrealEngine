// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#if WITH_CEF3
#include "Mac/MacSystemIncludes.h"

#import <objc/runtime.h>

#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#import "include/cef_application_mac.h"
PRAGMA_ENABLE_DEPRECATION_WARNINGS
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")

@interface NSApplication (CEF3UtilsApplication) < CefAppProtocol >
{
}
-(void)cef3UtilsSendEvent : (NSEvent*)event;
@end

@implementation NSApplication (CEF3UtilsApplication)
static BOOL bGHandlingSendEvent;

// Since objc no longer supports class_poseAs, do black magic to be able to override sendEvent in NSApplication
+ (void)load
{
	static dispatch_once_t DispatchToken;
	dispatch_once(&DispatchToken,
	^{
		Method OriginalMethod = class_getInstanceMethod([self class], @selector(sendEvent:));
		Method PatchedMethod = class_getInstanceMethod([self class], @selector(cef3UtilsSendEvent:));

		method_exchangeImplementations(OriginalMethod, PatchedMethod);
	});
}

- (BOOL)isHandlingSendEvent
{
	NSNumber* Property = objc_getAssociatedObject(self, @selector(isHandlingSendEvent));
	return [Property boolValue];
}

-(void)setHandlingSendEvent:(BOOL)handlingSendEvent
{
	objc_setAssociatedObject(self, @selector(isHandlingSendEvent), [NSNumber numberWithBool: handlingSendEvent], OBJC_ASSOCIATION_RETAIN_NONATOMIC);
}

// This will be swizzled with NSApplication -sendEvent:
-(void)cef3UtilsSendEvent : (NSEvent*)event
{
	CefScopedSendingEvent sendingEventScoper;
	[self cef3UtilsSendEvent : event];
}
@end
#endif
