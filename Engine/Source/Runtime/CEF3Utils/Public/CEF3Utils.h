// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

#if WITH_CEF3
namespace CEF3Utils
{
	/**
	 * Load the required modules for CEF3, returns false if we fail to load the cef library
	 */
	CEF3UTILS_API bool LoadCEF3Modules(bool bIsMainApp);

	/**
	 * Unload the required modules for CEF3
	 */
	CEF3UTILS_API void UnloadCEF3Modules();

#if PLATFORM_WINDOWS
	/**
	 * Get the module (dll) handle to the loaded CEF3 module, will be null if not loaded
	 */
	CEF3UTILS_API void* GetCEF3ModuleHandle();
#endif

	/**
	  * Get the module loaded path
	 */
	CEF3UTILS_API FString GetCEF3ModulePath();

	/**
	 * Move the current cef3.log file to a backup file, so CEF makes a new log when it starts up.
	 * This backup file is then cleaned up by the logic in FMaintenance::DeleteOldLogs()
	 */
	CEF3UTILS_API void BackupCEF3Logfile(const FString& LogFilePath);
};
#endif //WITH_CEF3
